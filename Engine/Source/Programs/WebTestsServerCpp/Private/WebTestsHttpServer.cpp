// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebTestsServer.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Templates/Atomic.h"
#include "HttpPath.h"
#include "HttpRequestHandler.h"
#include "HttpRouteHandle.h"
#include "HttpServerConstants.h"
#include "HttpServerModule.h"
#include "HttpServerRequest.h"
#include "HttpServerResponse.h"
#include "IHttpRouter.h"
#include "Misc/Compression.h"

DEFINE_LOG_CATEGORY_STATIC(LogHttpServer, Log, All);

// ---------------------------------------------------------------------------
// Helper: build a JSON response with Content-Type application/json
// ---------------------------------------------------------------------------
static TUniquePtr<FHttpServerResponse> JsonResponse(const FString& JsonBody, EHttpServerResponseCodes Code = EHttpServerResponseCodes::Ok)
{
	FTCHARToUTF8 Utf8(*JsonBody);
	TArray<uint8> Bytes((const uint8*)Utf8.Get(), Utf8.Length());
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(MoveTemp(Bytes), TEXT("application/json"));
	Response->Code = Code;
	return Response;
}

// ---------------------------------------------------------------------------
// Helper: get a named path param as int32 (returns 0 on failure)
// ---------------------------------------------------------------------------
static int32 PathParamInt(const FHttpServerRequest& Request, const FString& Name)
{
	const FString* Val = Request.PathParams.Find(Name);
	if (!Val) return 0;
	return FCString::Atoi(**Val);
}

// ---------------------------------------------------------------------------
// Log helper: one line per request handled
// ---------------------------------------------------------------------------
static FString VerbToString(EHttpServerRequestVerbs Verb)
{
	switch (Verb)
	{
	case EHttpServerRequestVerbs::VERB_GET:     return TEXT("GET");
	case EHttpServerRequestVerbs::VERB_POST:    return TEXT("POST");
	case EHttpServerRequestVerbs::VERB_PUT:     return TEXT("PUT");
	case EHttpServerRequestVerbs::VERB_PATCH:   return TEXT("PATCH");
	case EHttpServerRequestVerbs::VERB_DELETE:  return TEXT("DELETE");
	case EHttpServerRequestVerbs::VERB_OPTIONS: return TEXT("OPTIONS");
	case EHttpServerRequestVerbs::VERB_HEAD:    return TEXT("HEAD");
	case EHttpServerRequestVerbs::VERB_TRACE:   return TEXT("TRACE");
	case EHttpServerRequestVerbs::VERB_CONNECT: return TEXT("CONNECT");
	default:                                    return TEXT("UNKNOWN");
	}
}

static void LogRequest(const FHttpServerRequest& Request, int32 ResponseCode)
{
	UE_LOGF(LogHttpServer, Display, "%ls %ls => %d",
		*VerbToString(Request.Verb),
		*Request.FullPath.GetPath(),
		ResponseCode);
}

// ---------------------------------------------------------------------------
// Handlers
// ---------------------------------------------------------------------------

// GET/POST/PUT/PATCH/DELETE/OPTIONS /webtests/httptests/methods
static bool HandleMethods(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	LogRequest(Request, 200);
	OnComplete(JsonResponse(TEXT("{}")));
	return true;
}

// GET /webtests/httptests/query_with_params/?var_int=X&var_str=Y
static bool HandleQueryWithParams(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	const FString* VarIntStr = Request.QueryParams.Find(TEXT("var_int"));
	const FString* VarStr    = Request.QueryParams.Find(TEXT("var_str"));

	int32 VarInt = VarIntStr ? FCString::Atoi(**VarIntStr) : 0;
	FString StrVal = VarStr ? *VarStr : TEXT("");

	FString Body = FString::Printf(TEXT("{\"var_str\":\"%s\",\"var_int\":%d}"), *StrVal, VarInt);
	LogRequest(Request, 200);
	OnComplete(JsonResponse(Body));
	return true;
}

// GET /webtests/httptests/get_data_without_chunks/:bytes_number/:repeat_at/
static bool HandleGetDataWithoutChunks(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	int32 BytesNumber = PathParamInt(Request, TEXT("bytes_number"));
	int32 RepeatAt    = PathParamInt(Request, TEXT("repeat_at"));

	if (BytesNumber <= 0 || RepeatAt <= 0 || RepeatAt > 10)
	{
		OnComplete(FHttpServerResponse::Error(EHttpServerResponseCodes::BadRequest));
		return true;
	}

	TArray<uint8> Data;
	Data.Reserve(BytesNumber);
	for (int32 i = 0; i < BytesNumber; ++i)
	{
		Data.Add(static_cast<uint8>('0' + (i % RepeatAt)));
	}

	LogRequest(Request, 200);
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(MoveTemp(Data), TEXT("application/octet-stream"));
	Response->Code = EHttpServerResponseCodes::Ok;
	OnComplete(MoveTemp(Response));
	return true;
}

// GET /webtests/httptests/get_data_gzip/:bytes_number/:repeat_at/
static bool HandleGetDataGzip(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	int32 BytesNumber = PathParamInt(Request, TEXT("bytes_number"));
	int32 RepeatAt    = PathParamInt(Request, TEXT("repeat_at"));

	if (BytesNumber <= 0 || RepeatAt <= 0 || RepeatAt > 10)
	{
		OnComplete(FHttpServerResponse::Error(EHttpServerResponseCodes::BadRequest));
		return true;
	}

	TArray<uint8> RawData;
	RawData.Reserve(BytesNumber);
	for (int32 i = 0; i < BytesNumber; ++i)
	{
		RawData.Add(static_cast<uint8>('0' + (i % RepeatAt)));
	}

	int64 CompressBound = 0;
	FCompression::CompressMemoryBound(NAME_Gzip, CompressBound, (int64)RawData.Num());
	TArray<uint8> Compressed;
	Compressed.SetNumUninitialized((int32)CompressBound);
	int64 CompressedSize = CompressBound;
	FCompression::CompressMemory(NAME_Gzip, Compressed.GetData(), CompressedSize, RawData.GetData(), (int64)RawData.Num());
	Compressed.SetNum((int32)CompressedSize);

	LogRequest(Request, 200);
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(MoveTemp(Compressed), TEXT("application/octet-stream"));
	Response->Code = EHttpServerResponseCodes::Ok;
	Response->Headers.Add(TEXT("Content-Encoding"), { TEXT("gzip") });
	OnComplete(MoveTemp(Response));
	return true;
}

// ---------------------------------------------------------------------------
// Background runnable: streams chunks one at a time via StreamingBodyQueue,
// with per-chunk latency sleep, so the client receives data incrementally.
// ---------------------------------------------------------------------------
class FStreamingDownloadRunnable : public FRunnable
{
public:
	FStreamingDownloadRunnable(int32 InChunks, int32 InChunkSize, int32 InLatencySecs,
		TSharedPtr<TQueue<TArray<uint8>, EQueueMode::Spsc>> InQueue,
		TSharedPtr<TAtomic<bool>> InComplete)
		: Chunks(InChunks), ChunkSize(InChunkSize), LatencySecs(InLatencySecs)
		, Queue(MoveTemp(InQueue)), Complete(MoveTemp(InComplete))
	{}

	virtual uint32 Run() override
	{
		UE_LOGF(LogHttpServer, Display, "streaming_download: %d chunks x %d bytes, latency %ds each", Chunks, ChunkSize, LatencySecs);

		TArray<uint8> ChunkData;
		ChunkData.SetNumUninitialized(ChunkSize);
		FMemory::Memset(ChunkData.GetData(), (uint8)'d', ChunkSize);

		for (int32 i = 0; i < Chunks; ++i)
		{
			if (LatencySecs > 0)
			{
				FPlatformProcess::SleepNoStats((float)LatencySecs);
			}
			Queue->Enqueue(ChunkData);
		}

		// Signal that all chunks have been enqueued (Release pairs with Acquire load in consumer)
		Complete->Store(true, EMemoryOrder::Relaxed);
		return 0;
	}

	virtual void Exit() override { delete this; }

private:
	int32 Chunks;
	int32 ChunkSize;
	int32 LatencySecs;
	TSharedPtr<TQueue<TArray<uint8>, EQueueMode::Spsc>> Queue;
	TSharedPtr<TAtomic<bool>> Complete;
};

// GET /webtests/httptests/streaming_download/:chunks/:chunk_size/:chunk_latency/
static bool HandleStreamingDownload(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	int32 Chunks      = PathParamInt(Request, TEXT("chunks"));
	int32 ChunkSize   = PathParamInt(Request, TEXT("chunk_size"));
	int32 LatencySecs = PathParamInt(Request, TEXT("chunk_latency"));

	if (Chunks <= 0 || ChunkSize <= 0)
	{
		OnComplete(FHttpServerResponse::Error(EHttpServerResponseCodes::BadRequest));
		return true;
	}

	LogRequest(Request, 200);

	int64 TotalSize = (int64)Chunks * ChunkSize;
	auto Queue    = MakeShared<TQueue<TArray<uint8>, EQueueMode::Spsc>>();
	auto Complete = MakeShared<TAtomic<bool>>(false);

	TUniquePtr<FHttpServerResponse> ResponseUnique = MakeUnique<FHttpServerResponse>();
	ResponseUnique->Code = EHttpServerResponseCodes::Ok;
	ResponseUnique->Headers.Add(TEXT("content-type"), { TEXT("application/octet-stream") });
	ResponseUnique->Headers.Add(TEXT("content-length"), { FString::FromInt((int32)TotalSize) });
	ResponseUnique->StreamingBodyQueue   = Queue;
	ResponseUnique->StreamingBodyComplete = Complete;

	OnComplete(MoveTemp(ResponseUnique));

	// FRunnable::Exit() deletes itself after thread finishes
	FStreamingDownloadRunnable* Runnable = new FStreamingDownloadRunnable(Chunks, ChunkSize, LatencySecs, Queue, Complete);
	FRunnableThread::Create(Runnable, TEXT("StreamingDownload"), 0, TPri_Normal);
	return true;
}

// POST /webtests/httptests/streaming_upload_post
static bool HandleStreamingUploadPost(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	LogRequest(Request, 200);
	UE_LOGF(LogHttpServer, Display, "  body: %d bytes", Request.Body.Num());
	OnComplete(JsonResponse(TEXT("{}")));
	return true;
}

// PUT /webtests/httptests/streaming_upload_put
static bool HandleStreamingUploadPut(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	LogRequest(Request, 200);
	UE_LOGF(LogHttpServer, Display, "  body: %d bytes", Request.Body.Num());
	OnComplete(JsonResponse(TEXT("{}")));
	return true;
}

// GET /webtests/httptests/redirect_from
static bool HandleRedirectFrom(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	LogRequest(Request, 302);
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Redirect;
	Response->Headers.Add(TEXT("Location"), { TEXT("/webtests/httptests/redirect_to") });
	OnComplete(MoveTemp(Response));
	return true;
}

// GET /webtests/httptests/redirect_to
static bool HandleRedirectTo(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	LogRequest(Request, 200);
	OnComplete(JsonResponse(TEXT("{}")));
	return true;
}

// ---------------------------------------------------------------------------
// Background runnable: sleeps N seconds, then enqueues response to game thread
// ---------------------------------------------------------------------------
class FMockLatencyRunnable : public FRunnable
{
public:
	FMockLatencyRunnable(int32 InSeconds, FHttpResultCallback InOnComplete)
		: Seconds(InSeconds), OnComplete(MoveTemp(InOnComplete))
	{}

	virtual uint32 Run() override
	{
		UE_LOGF(LogHttpServer, Display, "mock_latency: sleeping %ds", Seconds);
		FPlatformProcess::SleepNoStats((float)Seconds);
		EnqueueDeferredResponse(MoveTemp(OnComplete), JsonResponse(TEXT("{}")));
		return 0;
	}

	virtual void Exit() override { delete this; }

private:
	int32 Seconds;
	FHttpResultCallback OnComplete;
};

// GET /webtests/httptests/mock_latency/:latency/
static bool HandleMockLatency(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	int32 Latency = PathParamInt(Request, TEXT("latency"));
	LogRequest(Request, 200);
	FMockLatencyRunnable* Runnable = new FMockLatencyRunnable(Latency, OnComplete);
	FRunnableThread::Create(Runnable, TEXT("MockLatency"), 0, TPri_Normal);
	return true;
}

// GET,POST,PUT,DELETE /webtests/httptests/mock_status/:status_code/
static bool HandleMockStatus(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	int32 StatusCode = PathParamInt(Request, TEXT("status_code"));
	TUniquePtr<FHttpServerResponse> Response = JsonResponse(
		TEXT("{\"key_a\":\"value_a\",\"key_b\":\"value_b\"}"),
		static_cast<EHttpServerResponseCodes>(StatusCode));

	const TArray<FString>* RetryAfterValues = Request.Headers.Find(TEXT("Retry-After"));
	if (RetryAfterValues && RetryAfterValues->Num() > 0)
	{
		Response->Headers.Add(TEXT("Retry-After"), *RetryAfterValues);
	}

	LogRequest(Request, StatusCode);
	OnComplete(MoveTemp(Response));
	return true;
}

// POST /webtests/httptests/post_echo_body
static bool HandlePostEchoBody(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	LogRequest(Request, 200);
	TArray<uint8> Body = Request.Body;
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(MoveTemp(Body), TEXT("application/json"));
	Response->Code = EHttpServerResponseCodes::Ok;
	OnComplete(MoveTemp(Response));
	return true;
}

// ---------------------------------------------------------------------------
// Public entry point
// ---------------------------------------------------------------------------
void StartHttpServer(uint32 Port)
{
	TSharedPtr<IHttpRouter> Router = FHttpServerModule::Get().GetHttpRouter(Port, /*bFailOnBindFailure=*/true);
	if (!Router.IsValid())
	{
		UE_LOGF(LogHttpServer, Error, "Failed to bind HTTP router on port %d", Port);
		return;
	}

	using EVerb = EHttpServerRequestVerbs;
	const EVerb AllMethods = EVerb::VERB_GET | EVerb::VERB_POST | EVerb::VERB_PUT | EVerb::VERB_PATCH
	                       | EVerb::VERB_DELETE | EVerb::VERB_OPTIONS | EVerb::VERB_HEAD | EVerb::VERB_TRACE | EVerb::VERB_CONNECT;

	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/methods")),
		AllMethods,
		FHttpRequestHandler::CreateStatic(&HandleMethods));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/query_with_params/")),
		EVerb::VERB_GET,
		FHttpRequestHandler::CreateStatic(&HandleQueryWithParams));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/get_data_without_chunks/:bytes_number/:repeat_at/")),
		EVerb::VERB_GET,
		FHttpRequestHandler::CreateStatic(&HandleGetDataWithoutChunks));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/get_data_gzip/:bytes_number/:repeat_at/")),
		EVerb::VERB_GET,
		FHttpRequestHandler::CreateStatic(&HandleGetDataGzip));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/streaming_download/:chunks/:chunk_size/:chunk_latency/")),
		EVerb::VERB_GET,
		FHttpRequestHandler::CreateStatic(&HandleStreamingDownload));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/streaming_upload_post")),
		EVerb::VERB_POST,
		FHttpRequestHandler::CreateStatic(&HandleStreamingUploadPost));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/streaming_upload_put")),
		EVerb::VERB_PUT,
		FHttpRequestHandler::CreateStatic(&HandleStreamingUploadPut));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/redirect_from")),
		EVerb::VERB_GET,
		FHttpRequestHandler::CreateStatic(&HandleRedirectFrom));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/redirect_to")),
		EVerb::VERB_GET,
		FHttpRequestHandler::CreateStatic(&HandleRedirectTo));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/mock_latency/:latency/")),
		EVerb::VERB_GET,
		FHttpRequestHandler::CreateStatic(&HandleMockLatency));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/mock_status/:status_code/")),
		EVerb::VERB_GET | EVerb::VERB_POST | EVerb::VERB_PUT | EVerb::VERB_DELETE,
		FHttpRequestHandler::CreateStatic(&HandleMockStatus));
	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/post_echo_body")),
		EVerb::VERB_POST,
		FHttpRequestHandler::CreateStatic(&HandlePostEchoBody));

	FHttpServerModule::Get().StartAllListeners();
	UE_LOGF(LogHttpServer, Display, "HTTP server listening on http://0.0.0.0:%d/", Port);
}
