// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebTestsServer.h"
#include "Containers/Ticker.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
#include "HttpServerModule.h"
#include "WebSocketServerModule.h"
#include "RequiredProgramMainCPPInclude.h"

IMPLEMENT_APPLICATION(WebTestsServerCpp, "WebTestsServerCpp");

// ---------------------------------------------------------------------------
// Deferred response queue implementation (shared with WebTestsHttpServer.cpp)
// ---------------------------------------------------------------------------
static FCriticalSection GDeferredResponsesLock;
static TArray<FDeferredResponse> GDeferredResponses;

void EnqueueDeferredResponse(FHttpResultCallback&& OnComplete, TUniquePtr<FHttpServerResponse>&& Response)
{
	FScopeLock Lock(&GDeferredResponsesLock);
	GDeferredResponses.Add({ MoveTemp(OnComplete), MoveTemp(Response) });
}

void FlushDeferredResponses()
{
	TArray<FDeferredResponse> ToFlush;
	{
		FScopeLock Lock(&GDeferredResponsesLock);
		ToFlush = MoveTemp(GDeferredResponses);
	}
	for (FDeferredResponse& Deferred : ToFlush)
	{
		Deferred.OnComplete(MoveTemp(Deferred.Response));
	}
}

int32 RunWebTestsServer(int32 ArgC, TCHAR** ArgV)
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	ON_SCOPE_EXIT
	{
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}

	// Bind to 0.0.0.0 so remote test devices can reach the server via LAN IP
	GConfig->SetString(TEXT("HTTPServer.Listeners"), TEXT("DefaultBindAddress"), TEXT("any"), GEngineIni);

	StartHttpServer(8000);
	StartWebSocketServer(8001);

	const float TickInterval = 1.0f / 60.0f;
	while (!IsEngineExitRequested())
	{
		FlushDeferredResponses();
		FTSTicker::GetCoreTicker().Tick(TickInterval);
		FPlatformProcess::SleepNoStats(TickInterval);
	}

	FHttpServerModule::Get().StopAllListeners();
	FWebSocketServerModule::Get().StopAllServers();

	return 0;
}

#if !PLATFORM_MAC
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	return RunWebTestsServer(ArgC, ArgV);
}
#endif
