// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HttpResultCallback.h"
#include "HttpServerResponse.h"

// ---------------------------------------------------------------------------
// Deferred response queue — background threads enqueue (callback, response)
// pairs; the main tick loop drains them on the game thread.
// ---------------------------------------------------------------------------
struct FDeferredResponse
{
	FHttpResultCallback             OnComplete;
	TUniquePtr<FHttpServerResponse> Response;
};

void EnqueueDeferredResponse(FHttpResultCallback&& OnComplete, TUniquePtr<FHttpServerResponse>&& Response);
void FlushDeferredResponses();

void StartHttpServer(uint32 Port);
void StartWebSocketServer(uint32 Port);
int32 RunWebTestsServer(int32 ArgC, TCHAR** ArgV);
