// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#if PLATFORM_MAC

#include "WebTestsServer.h"

// Defined in CocoaThread.cpp — registers the current thread as the main run
// loop source so that MainThreadCall() from worker threads works correctly.
// Declared here directly to avoid including CocoaThread.h (ObjC syntax).
CORE_API void RegisterCurrentThreadAsMain();

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	RegisterCurrentThreadAsMain();
	return RunWebTestsServer(ArgC, ArgV);
}

#endif // PLATFORM_MAC
