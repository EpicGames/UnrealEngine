// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "IWebSocketClientConnection.h"
#include "IWebSocketServer.h"
#include "WebSocketServerModule.h"

void StartWebSocketServer(uint32 Port)
{
	TSharedPtr<IWebSocketServer> WsServer = FWebSocketServerModule::Get().GetWebSocketServer(Port);

	WsServer->OnConnected([](TSharedRef<IWebSocketClientConnection> Conn)
	{
		UE_LOGF(LogWebSocketServer, Display, "WS  CONNECT  %ls", *Conn->GetPath());
	});

	WsServer->OnMessage([](TSharedRef<IWebSocketClientConnection> Conn, const FString& Msg)
	{
		const FString Path = Conn->GetPath();
		UE_LOGF(LogWebSocketServer, Display, "WS  MESSAGE  path='%ls' msg='%ls'", *Path, *Msg);
		if (Path.EndsWith(TEXT("/echo/")))
		{
			UE_LOGF(LogWebSocketServer, Display, "WS  ECHO     sending back '%ls'", *Msg);
			Conn->SendText(Msg);
		}
		else if (Path.EndsWith(TEXT("/close_on_receive_message/")))
		{
			UE_LOGF(LogWebSocketServer, Display, "WS  CLOSE    closing connection");
			Conn->Close(1000);
		}
	});

	WsServer->OnDisconnected([](TSharedRef<IWebSocketClientConnection> Conn)
	{
		UE_LOGF(LogWebSocketServer, Display, "WS  DISCONNECT %ls", *Conn->GetPath());
	});

	FWebSocketServerModule::Get().StartAllServers();
	UE_LOGF(LogWebSocketServer, Display, "WebSocket server listening on ws://0.0.0.0:%d/", Port);
}
