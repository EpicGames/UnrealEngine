// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebTestsServerCpp : ModuleRules
{
	public WebTestsServerCpp(ReadOnlyTargetRules Target) : base(Target)
	{
		bUseUnity = false;

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ApplicationCore",
				"Projects",
				"HTTPServer",
				"WebSocketServer",
				"Sockets",
				"Json",
			});
	}
}
