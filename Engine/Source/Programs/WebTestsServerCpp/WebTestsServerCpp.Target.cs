// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class WebTestsServerCppTarget : TargetRules
{
	public WebTestsServerCppTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "WebTestsServerCpp";

		bBuildDeveloperTools = false;
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = true;
		bCompileICU = false;
		bIsBuildingConsoleApplication = true;
	}
}
