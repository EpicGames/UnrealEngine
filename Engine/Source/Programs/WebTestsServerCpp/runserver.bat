@echo off
setlocal

set SCRIPT_DIR=%~dp0
set UE_ROOT=%SCRIPT_DIR%..\..\..\..

echo Building WebTestsServerCpp...
call "%UE_ROOT%\Engine\Build\BatchFiles\Build.bat" WebTestsServerCpp Win64 Development
if %ERRORLEVEL% neq 0 (
    echo Build failed.
    exit /b %ERRORLEVEL%
)

echo Starting WebTestsServerCpp...
"%UE_ROOT%\Engine\Binaries\Win64\WebTestsServerCpp.exe" -log -NOLOGTIMES
