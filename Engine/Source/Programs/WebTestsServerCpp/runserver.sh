#!/bin/bash

SCRIPT_DIR="$(cd "$(dirname "${BASH_SOURCE[0]}")" && pwd)"
UE_ROOT="$SCRIPT_DIR/../../../.."

if [ "$(uname)" = "Darwin" ]; then
    PLATFORM="Mac"
else
    PLATFORM="Linux"
fi

echo "Building WebTestsServerCpp..."
"$UE_ROOT/Engine/Build/BatchFiles/RunUBT.sh" WebTestsServerCpp "$PLATFORM" Development
if [ $? -ne 0 ]; then
    echo "Build failed."
    exit 1
fi

echo "Starting WebTestsServerCpp..."
"$UE_ROOT/Engine/Binaries/$PLATFORM/WebTestsServerCpp" -log -NOLOGTIMES
